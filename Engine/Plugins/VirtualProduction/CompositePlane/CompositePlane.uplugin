{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Composite Plane",
	"Description": "Provides a cine camera actor for projecting textures and videos",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "CompositePlane",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}