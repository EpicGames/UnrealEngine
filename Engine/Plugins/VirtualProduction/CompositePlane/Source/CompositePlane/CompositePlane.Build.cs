// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CompositePlane : ModuleRules
{
	public CompositePlane(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
                "Core",
                "CoreUObject",
				"Engine",
                "Projects",
                "Slate",
				"SlateCore",
			}
			);

        if (Target.bBuildEditor)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[] {
						"EditorFramework",
                        "UnrealEd",
                        "PlacementMode",
                }
            );
        }
	}
}