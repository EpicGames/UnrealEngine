// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositePlane.h"
#include "CompositePlanePlacement.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FCompositePlaneModule"

void FCompositePlaneModule::StartupModule()
{
	FCompositePlanePlacement::RegisterPlacement();
}

void FCompositePlaneModule::ShutdownModule()
{
	FCompositePlanePlacement::UnregisterPlacement();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCompositePlaneModule, CompositePlane)
