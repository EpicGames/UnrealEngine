// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositePlanePlacement.h"
#include "Engine/Blueprint.h"
#include "IPlacementModeModule.h"
#include "ActorFactories/ActorFactoryBlueprint.h"

#define LOCTEXT_NAMESPACE "FCompositePlaneModule"

void FCompositePlanePlacement::RegisterPlacement()
{
	UBlueprint* CompositePlane = Cast<UBlueprint>(FSoftObjectPath(TEXT("/CompositePlane/BP_CineCameraProj.BP_CineCameraProj")).TryLoad());
	if (CompositePlane == nullptr)
	{
		return;
	}

	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	const FPlacementCategoryInfo* Info = PlacementModeModule.GetRegisteredPlacementCategory(FName("Cinematic"));
	if (Info == nullptr)
	{
		return;
	}

	FPlaceableItem* BPPlacement = new FPlaceableItem(
		*UActorFactoryBlueprint::StaticClass(),
		FAssetData(CompositePlane, true),
		FName(""),
		FName(""),
		TOptional<FLinearColor>(),
		TOptional<int32>(),
		NSLOCTEXT("PlacementMode", "Composite Plane", "Composite Plane")
	);

	IPlacementModeModule::Get().RegisterPlaceableItem(Info->UniqueHandle, MakeShareable(BPPlacement));
}

void FCompositePlanePlacement::UnregisterPlacement()
{
	if (IPlacementModeModule::IsAvailable())
	{
		IPlacementModeModule::Get().UnregisterPlacementCategory(FName("Cinematic"));
	}
}

#undef LOCTEXT_NAMESPACE
