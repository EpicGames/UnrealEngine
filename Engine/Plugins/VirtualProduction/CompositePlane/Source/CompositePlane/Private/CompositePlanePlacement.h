// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


class FCompositePlanePlacement
{
public:
	static void RegisterPlacement();
	static void UnregisterPlacement();
};
